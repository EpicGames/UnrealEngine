// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/TaskGraphInterfaces.h"
#include "Containers/SharedString.h"
#include "DerivedDataBuildFunctionFactory.h"
#include "HAL/IConsoleManager.h"
#include "HAL/FileManager.h"
#include "HAL/PlatformProcess.h"
#include "Misc/CommandLine.h"
#include "Misc/ConfigCacheIni.h"
#include "Misc/FileHelper.h"
#include "Misc/Guid.h"
#include "Misc/Paths.h"
#include "ImageCore.h"
#include "Interfaces/ITargetPlatform.h"
#include "Interfaces/ITargetPlatformManagerModule.h"
#include "Interfaces/ITextureFormat.h"
#include "Interfaces/ITextureFormatModule.h"
#include "Modules/ModuleManager.h"
#include "PixelFormat.h"
#include "Serialization/CompactBinary.h"
#include "Serialization/CompactBinaryWriter.h"
#include "Tasks/Task.h"
#include "TextureBuildFunction.h"
#include "TextureCompressorModule.h"

#include "astc_thunk.h"

static void* FMemory_AstcThunk_Malloc(size_t Size, size_t Alignment)
{ 
	return FMemory::Malloc( Size ? Size : 1, Alignment ); 
}

static void FMemory_AstcThunk_Free(void *Ptr)
{
	FMemory::Free( Ptr ); 
}


/****
* 
* TextureFormatASTC runs the ARM astcenc
* 
*****/

// increment this if you change anything that will affect compression in this file
// Avoid changing this! Rebuilding textures is usually because something changed in encoding
// which causes a huge patch. Try and make the new code only affect textures that opt in to the new
// behavior.
#define BASE_ASTC_FORMAT_VERSION 48

#define MAX_QUALITY_BY_SIZE 4
#define MAX_QUALITY_BY_SPEED 3

/**

"Quality" in this file is ETextureCompressionQuality-1

so a "3" here == High == 6x6

enum ETextureCompressionQuality : int
{
	TCQ_Default = 0		UMETA(DisplayName="Default"),
	TCQ_Lowest = 1		UMETA(DisplayName="Lowest (ASTC 12x12)"),
	TCQ_Low = 2			UMETA(DisplayName="Low (ASTC 10x10)"),
	TCQ_Medium = 3		UMETA(DisplayName="Medium (ASTC 8x8)"),
	TCQ_High= 4			UMETA(DisplayName="High (ASTC 6x6)"),
	TCQ_Highest = 5		UMETA(DisplayName="Highest (ASTC 4x4)"),
	TCQ_MAX,
};


**/


DEFINE_LOG_CATEGORY_STATIC(LogTextureFormatASTC, Log, All);

class FASTCTextureBuildFunction final : public FTextureBuildFunction
{
	const UE::FUtf8SharedString& GetName() const final
	{
		static const UE::FUtf8SharedString Name(UTF8TEXTVIEW("ASTCTexture"));
		return Name;
	}

	void GetVersion(UE::DerivedData::FBuildVersionBuilder& Builder, ITextureFormat*& OutTextureFormatVersioning) const final
	{
		static FGuid Version(TEXT("4788dab5-b99c-479f-bc34-6d7df1cf30e5"));
		Builder << Version;
		OutTextureFormatVersioning = FModuleManager::GetModuleChecked<ITextureFormatModule>(TEXT("TextureFormatASTC")).GetTextureFormat();
	}
};

/**
 * Macro trickery for supported format names.
 */
#define ENUM_SUPPORTED_FORMATS(op) \
	op(ASTC_RGB) \
	op(ASTC_RGBA) \
	op(ASTC_RGBAuto) \
	op(ASTC_RGBA_HQ) \
	op(ASTC_RGB_HDR) \
	op(ASTC_NormalLA) \
	op(ASTC_NormalAG) \
	op(ASTC_NormalRG) \
	op(ASTC_NormalRG_Precise) // Encoded as LA for precision, mapped to RG at runtime. RHI needs to support PF_ASTC_*_NORM_RG formats (requires runtime swizzle)

	#define DECL_FORMAT_NAME(FormatName) static FName GTextureFormatName##FormatName = FName(TEXT(#FormatName));
		ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME);
	#undef DECL_FORMAT_NAME

	#define DECL_FORMAT_NAME_ENTRY(FormatName) GTextureFormatName##FormatName ,
		static FName GSupportedTextureFormatNames[] =
		{
			ENUM_SUPPORTED_FORMATS(DECL_FORMAT_NAME_ENTRY)
		};
	#undef DECL_FORMAT_NAME_ENTRY
#undef ENUM_SUPPORTED_FORMATS

// ASTC file header format
#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(push, 4)
#endif

	#define ASTC_MAGIC_CONSTANT 0x5CA1AB13
	struct FASTCHeader
	{
		uint32 Magic;
		uint8  BlockSizeX;
		uint8  BlockSizeY;
		uint8  BlockSizeZ;
		uint8  TexelCountX[3];
		uint8  TexelCountY[3];
		uint8  TexelCountZ[3];
	};

#if PLATFORM_SUPPORTS_PRAGMA_PACK
	#pragma pack(pop)
#endif

static bool IsNormalMapFormat(FName TextureFormatName)
{
	return
		TextureFormatName == GTextureFormatNameASTC_NormalAG ||
		TextureFormatName == GTextureFormatNameASTC_NormalRG ||
		TextureFormatName == GTextureFormatNameASTC_NormalLA ||
		TextureFormatName == GTextureFormatNameASTC_NormalRG_Precise;
}

static bool IsHDRFormat(FName TextureFormatName)
{
	return TextureFormatName == GTextureFormatNameASTC_RGB_HDR;
}

static bool IsRDOEncode(const FTextureBuildSettings& InBuildSettings)
{
	if (InBuildSettings.AstcEncVersion == NAME_None)
	{
		// We don't support RDO until 5.0.1+
		return false;
	}

	if (!IsHDRFormat(InBuildSettings.TextureFormatName))
	{
		// We use whatever settings they've specified for Oodle.
		if (InBuildSettings.bOodleUsesRDO && InBuildSettings.OodleRDO != 0)
		{
			return true;
		}
	}
	return false;
}


static int32 GetDefaultCompressionBySizeValue(FCbObjectView InFormatConfigOverride)
{
	// this is code duped between TextureFormatASTC and TextureFormatISPC
	if (InFormatConfigOverride)
	{
		// If we have an explicit format config, then use it directly
		FCbFieldView FieldView = InFormatConfigOverride.FindView("DefaultASTCQualityBySize");
		checkf(FieldView.HasValue(), TEXT("Missing DefaultASTCQualityBySize key from FormatConfigOverride"));
		int32 CompressionModeValue = FieldView.AsInt32();
		checkf(!FieldView.HasError(), TEXT("Failed to parse DefaultASTCQualityBySize value from FormatConfigOverride"));
		return CompressionModeValue;
	}
	else
	{
		// default of 3 == 6x6

		auto GetCompressionModeValue = []() {
			// start at default quality, then lookup in .ini file
			int32 CompressionModeValue = 3;
			GConfig->GetInt(TEXT("/Script/UnrealEd.CookerSettings"), TEXT("DefaultASTCQualityBySize"), CompressionModeValue, GEngineIni);
	
			FParse::Value(FCommandLine::Get(), TEXT("-astcqualitybysize="), CompressionModeValue);
			
			return FMath::Min<uint32>(CompressionModeValue, MAX_QUALITY_BY_SIZE);
		};

		static int32 CompressionModeValue = GetCompressionModeValue();

		return CompressionModeValue;
	}
}

static int32 GetDefaultCompressionBySizeValueHQ(FCbObjectView InFormatConfigOverride)
{
	// this is code duped between TextureFormatASTC and TextureFormatISPC
	if (InFormatConfigOverride)
	{
		// If we have an explicit format config, then use it directly
		FCbFieldView FieldView = InFormatConfigOverride.FindView("DefaultASTCQualityBySizeHQ");
		checkf(FieldView.HasValue(), TEXT("Missing DefaultASTCQualityBySizeHQ key from FormatConfigOverride"));
		int32 CompressionModeValue = FieldView.AsInt32();
		checkf(!FieldView.HasError(), TEXT("Failed to parse DefaultASTCQualityBySizeHQ value from FormatConfigOverride"));
		return CompressionModeValue;
	}
	else
	{
		// default of 4 == 4x4

		auto GetCompressionModeValue = []() {
			// start at default quality, then lookup in .ini file
			int32 CompressionModeValue = 4;
			GConfig->GetInt(TEXT("/Script/UnrealEd.CookerSettings"), TEXT("DefaultASTCQualityBySizeHQ"), CompressionModeValue, GEngineIni);
	
			FParse::Value(FCommandLine::Get(), TEXT("-astcqualitybysizehq="), CompressionModeValue);
			
			return FMath::Min<uint32>(CompressionModeValue, MAX_QUALITY_BY_SIZE);
		};

		static int32 CompressionModeValue = GetCompressionModeValue();

		return CompressionModeValue;
	}
}

static int32 GetDefaultCompressionBySpeedValue(FCbObjectView InFormatConfigOverride)
{
	if (InFormatConfigOverride)
	{
		// If we have an explicit format config, then use it directly
		FCbFieldView FieldView = InFormatConfigOverride.FindView("DefaultASTCQualityBySpeed");
		checkf(FieldView.HasValue(), TEXT("Missing DefaultASTCQualityBySpeed key from FormatConfigOverride"));
		int32 CompressionModeValue = FieldView.AsInt32();
		checkf(!FieldView.HasError(), TEXT("Failed to parse DefaultASTCQualityBySpeed value from FormatConfigOverride"));
		return CompressionModeValue;
	}
	else
	{

		// default of 2 == ASTCENC_PRE_MEDIUM

		auto GetCompressionModeValue = []() {
			// start at default quality, then lookup in .ini file
			int32 CompressionModeValue = 2;
			GConfig->GetInt(TEXT("/Script/UnrealEd.CookerSettings"), TEXT("DefaultASTCQualityBySpeed"), CompressionModeValue, GEngineIni);
	
			FParse::Value(FCommandLine::Get(), TEXT("-astcqualitybyspeed="), CompressionModeValue);
			
			return FMath::Min<uint32>(CompressionModeValue, MAX_QUALITY_BY_SPEED);
		};

		static int32 CompressionModeValue = GetCompressionModeValue();

		return CompressionModeValue;
	}
}

static EPixelFormat GetQualityFormat(const FTextureBuildSettings& BuildSettings)
{
	// code dupe between TextureFormatASTC  and TextureFormatISPC

	const FCbObjectView& InFormatConfigOverride = BuildSettings.FormatConfigOverride;
	int32 OverrideSizeValue= BuildSettings.CompressionQuality;

	bool bIsNormalMap = IsNormalMapFormat(BuildSettings.TextureFormatName);

	if ( bIsNormalMap )
	{
		// normal map hard coded to always use 6x6 currently
		//	ignores per-texture quality

		if ( BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG_Precise )
		{
			return PF_ASTC_6x6_NORM_RG;
		}
		else
		{
			return PF_ASTC_6x6;
		}
	}
	else if (BuildSettings.bVirtualStreamable)
	{
		return PF_ASTC_4x4;		
	}

	// CompressionQuality value here is ETextureCompressionQuality minus 1
	
	bool bIsHQ = BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGBA_HQ;
	bool bHDRFormat = IsHDRFormat(BuildSettings.TextureFormatName);
	
	if ( OverrideSizeValue < 0 )
	{
		if ( bIsHQ )
		{
			OverrideSizeValue = GetDefaultCompressionBySizeValueHQ(InFormatConfigOverride);
		}
		else
		{
			OverrideSizeValue = GetDefaultCompressionBySizeValue(InFormatConfigOverride);
		}
	}

	// convert to a string
	EPixelFormat Format = PF_Unknown;
	if (bHDRFormat)
	{
		switch (OverrideSizeValue)
		{
			case 0:	Format = PF_ASTC_12x12_HDR; break;
			case 1:	Format = PF_ASTC_10x10_HDR; break;
			case 2:	Format = PF_ASTC_8x8_HDR; break;
			case 3:	Format = PF_ASTC_6x6_HDR; break;
			case 4:	Format = PF_ASTC_4x4_HDR; break;
			default: UE_LOGF(LogTextureFormatASTC, Fatal, "Max quality higher than expected");
		}
	}
	else
	{
		switch (OverrideSizeValue)
		{
			case 0:	Format = PF_ASTC_12x12; break;
			case 1:	Format = PF_ASTC_10x10; break;
			case 2:	Format = PF_ASTC_8x8; break;
			case 3:	Format = PF_ASTC_6x6; break;
			case 4:	Format = PF_ASTC_4x4; break;
			default: UE_LOGF(LogTextureFormatASTC, Fatal, "Max quality higher than expected");
		}
	}
	return Format;
}

static bool IsASTCPixelFormatHDR(EPixelFormat PF)
{
	switch (PF)
	{
	case PF_ASTC_4x4_HDR:
	case PF_ASTC_6x6_HDR:
	case PF_ASTC_8x8_HDR:
	case PF_ASTC_10x10_HDR:
	case PF_ASTC_12x12_HDR:
		{
			return true;
		}
	}
	return false;
}


struct FAstcEncThunk
{
	UE::FMutex LoaderLock;
	bool bHasAttemptedLoad = false;
	void* LibHandle = nullptr;

	AstcThunk_CreateFnType* Create = nullptr;
	AstcThunk_DoWorkFnType* DoWork = nullptr;
	AstcThunk_DestroyFnType* Destroy = nullptr;
};

static bool ASTCEnc_Compress(
	const FAstcEncThunk* Thunk,
	const FImage& InImage,
	const FTextureBuildSettings& BuildSettings,
	const FIntVector3& InMip0Dimensions,
	int32 InMip0NumSlicesNoDepth,
	FStringView DebugTexturePathName,
	bool bImageHasAlphaChannel,
	FCompressedImage2D& OutCompressedImage)
{
	bool bHDRImage = IsHDRFormat(BuildSettings.TextureFormatName);
	// DestGamma is how the texture will be bound to GPU
	bool bSRGB = BuildSettings.GetDestGammaSpace() == EGammaSpace::sRGB;
	check( !bHDRImage || !bSRGB );

	// Get Raw Image Data from passed in FImage & convert to BGRA8 or RGBA16F
	// note: wasteful, often copies image to same format
	FImage Image;
	InImage.CopyTo(Image, bHDRImage ? ERawImageFormat::RGBA16F : ERawImageFormat::BGRA8, BuildSettings.GetDestGammaSpace());

	if (bHDRImage)
	{
		// ASTC can encode floats that BC6H can't
		//  but still clamp as if we were BC6H, so that the same output is made
		// (eg. ASTC can encode A but BC6 can't; we stuff 1 in A here)
		FImageCore::SanitizeFloat16AndSetAlphaOpaqueForBC6H(Image);
	}

	bool bIsNormalMap = IsNormalMapFormat(BuildSettings.TextureFormatName);
		
	// Determine the compressed pixel format and compression parameters
	EPixelFormat CompressedPixelFormat = GetQualityFormat(BuildSettings);

	FAstcEncThunk_CreateParams CreateParams;

	CreateParams.Flags = EAstcEncThunk_Flags::NONE;
	if (bIsNormalMap)
	{
		CreateParams.Flags |= EAstcEncThunk_Flags::NORMAL_MAP;
	}

	if (!bHDRImage &&
		IsRDOEncode(BuildSettings))
	{
		CreateParams.Flags |= EAstcEncThunk_Flags::LZ_RDO;
	}

	CreateParams.Profile = (bHDRImage ? EAstcEncThunk_Profile::HDR_RGB_LDR_A : (bSRGB ? EAstcEncThunk_Profile::LDR_SRGB : EAstcEncThunk_Profile::LDR));

	CreateParams.Quality = EAstcEncThunk_Quality::FAST;
	switch (GetDefaultCompressionBySpeedValue(BuildSettings.FormatConfigOverride))
	{
		case 0:	CreateParams.Quality = EAstcEncThunk_Quality::FASTEST; break;
		case 1:	CreateParams.Quality = EAstcEncThunk_Quality::FAST; break;
		case 2:	CreateParams.Quality = EAstcEncThunk_Quality::MEDIUM; break;
		case 3:	CreateParams.Quality = EAstcEncThunk_Quality::THOROUGH; break;
		default: UE_LOGF(LogTextureFormatASTC, Fatal, "ASTC speed quality higher than expected");
	}

	CreateParams.BlockSize = GPixelFormats[CompressedPixelFormat].BlockSizeX;

	CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_R;
	CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
	CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_B;
	CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_A;

	if (bHDRImage)
	{
		// BC6H does not support A, so we remove it to match
		CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
	}
	else if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGB ||
		BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGBA || 
		BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGBAuto || 
		BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGBA_HQ)
	{
		if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_RGB || !bImageHasAlphaChannel)
		{
			// even if Name was RGBA we still use the RGB profile if !bImageHasAlphaChannel
			//	so that "Compress Without Alpha" can force us to opaque

			// we need to set alpha to opaque here
			// can do it using "1" in the bgra swizzle to astcenc
			CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
		}

		// source is BGRA
		CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_B;
		CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_R;
	}
	else if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalAG)
	{
		// note that DXT5n processing does "1g0r"
		CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_1;
		CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
		CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_0;
		CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_B; // source is BGRA

		CreateParams.bDbLimitGreaterThan60 = true;

		CreateParams.ErrorWeightR = 0.0f;
		CreateParams.ErrorWeightG = 1.0f;
		CreateParams.ErrorWeightB = 0.0f;
		CreateParams.ErrorWeightA = 1.0f;
	}
	else if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG)
	{
		CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_B; // source is BGRA
		CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
		CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_0;
		CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;

		CreateParams.bDbLimitGreaterThan60 = true;

		CreateParams.ErrorWeightR = 1.0f;
		CreateParams.ErrorWeightG = 1.0f;
		CreateParams.ErrorWeightB = 0.0f;
		CreateParams.ErrorWeightA = 0.0f;
	}
	else if (BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalLA || BuildSettings.TextureFormatName == GTextureFormatNameASTC_NormalRG_Precise)
	{
		// L+A mode: rrrg
		CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_B;
		CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_B;
		CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_B;
		CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_G;

		CreateParams.bDbLimitGreaterThan60 = true;

		CreateParams.ErrorWeightR = 1.0f;
		CreateParams.ErrorWeightG = 0.0f;
		CreateParams.ErrorWeightB = 0.0f;
		CreateParams.ErrorWeightA = 1.0f;
	}
	else
	{
		check(false);
	}

	if (CreateParams.Flags & EAstcEncThunk_Flags::LZ_RDO)
	{
		CreateParams.LZRdoLambda = BuildSettings.OodleRDO;
	}

	// Set up output image
	{
		const int AlignedSizeX = AlignArbitrary(Image.SizeX, CreateParams.BlockSize);
		const int AlignedSizeY = AlignArbitrary(Image.SizeY, CreateParams.BlockSize);
		const int WidthInBlocks = AlignedSizeX / CreateParams.BlockSize;
		const int HeightInBlocks = AlignedSizeY / CreateParams.BlockSize;
		const int64 SizePerSlice = (int64)WidthInBlocks * HeightInBlocks * 16;
		OutCompressedImage.RawData.AddUninitialized(SizePerSlice * Image.NumSlices);

		CreateParams.OutputImageBuffer = OutCompressedImage.RawData.GetData();
		CreateParams.OutputImageBufferSize = OutCompressedImage.RawData.Num();
	}

	// Set up input image.
	TArray<uint8*, TInlineAllocator<6>> ImageSrcData;

	{
		ImageSrcData.Reserve(Image.NumSlices);

		for (int32 SliceIdx = 0; SliceIdx < Image.NumSlices; SliceIdx++)
		{
			FImageView Slice = Image.GetSlice(SliceIdx);
			uint8* SliceData;
			if (bHDRImage)
			{
				SliceData = (uint8*)Slice.AsRGBA16F().GetData();
			}
			else
			{
				SliceData = (uint8*)Slice.AsBGRA8().GetData();
			}
			ImageSrcData.Add(SliceData);
		}
	
		CreateParams.SizeX = Image.SizeX;
		CreateParams.SizeY = Image.SizeY;
		CreateParams.NumSlices = Image.NumSlices;
		CreateParams.ImageSlices = (void**)ImageSrcData.GetData();
		CreateParams.ImageDataType = (bHDRImage ? EAstcEncThunk_Type::F16 : EAstcEncThunk_Type::U8);		
	}

	//
	// Find a good number of tasks to divide the encode up. We try and make it so it's roughly 256x256 tiles per task,
	// but we also don't want to go too high because these aren't exactly cheap w/r/t memory internally to astcenc.
	//
	{
		uint32 ChunksX = FMath::DivideAndRoundUp(CreateParams.SizeX, 256U);
		uint32 ChunksY = FMath::DivideAndRoundUp(CreateParams.SizeY, 256U);
		uint32 ChunksZ = CreateParams.NumSlices;

		CreateParams.TaskCount = ChunksX * ChunksY * ChunksZ;

		uint32 WorkerThreadCount = FMath::Max(1, FTaskGraphInterface::Get().GetNumWorkerThreads());
		CreateParams.TaskCount = FMath::Min(CreateParams.TaskCount, WorkerThreadCount);
	}

	AstcEncThunk_Context Context;
	char const* ThunkError = Thunk->Create(CreateParams, &Context);
	if (ThunkError)
	{
		UE_LOGF(LogTextureFormatASTC, Error, "Failed to create astcenc thunk: %ls", ANSI_TO_TCHAR(ThunkError));
		Thunk->Destroy(Context);
		return false;
	}

	TArray<char const*, TInlineAllocator<8>> Results;
	Results.SetNumZeroed(CreateParams.TaskCount);

	TArray<UE::Tasks::FTask, TInlineAllocator<8>> EncodeTasks;
	EncodeTasks.Reserve(CreateParams.TaskCount);
	
	// Launch the other tasks, but keep one to run inline.
	for (uint32 TaskIndex = 1; TaskIndex < CreateParams.TaskCount; TaskIndex++)
	{
		UE::Tasks::FTask Task = UE::Tasks::Launch(TEXT("ASTCWorker"), [TaskIndex, &Context, Thunk, &Results]()
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(ASTCCompressWorker);
				Results[TaskIndex] = Thunk->DoWork(Context, TaskIndex);
			});

		EncodeTasks.Add(Task);
	}

	// Do our own task, always index 0.
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ASTCCompressInline);
		Results[0] = Thunk->DoWork(Context, 0);
	}

	UE::Tasks::Wait(EncodeTasks);

	Thunk->Destroy(Context);
	
	bool bSucceeded = true;
	for (uint32 TaskIndex = 0; TaskIndex < CreateParams.TaskCount; TaskIndex++)
	{
		if (Results[TaskIndex])
		{
			UE_LOGF(LogTextureFormatASTC, Error, "Astcenc Thunk DoWork has failed: %ls", ANSI_TO_TCHAR(Results[TaskIndex]));
			bSucceeded = false;
			break;
		}
	}
	
	if (bSucceeded)
	{
		OutCompressedImage.SizeX = Image.SizeX;
		OutCompressedImage.SizeY = Image.SizeY;
		OutCompressedImage.NumSlicesWithDepth = Image.NumSlices;
		OutCompressedImage.PixelFormat = CompressedPixelFormat;
		return true;
	}
	else
	{
		return false;
	}
}


/**
 * ASTC texture format handler.
 */
class FTextureFormatASTC : public ITextureFormat
{
public:
	FTextureFormatASTC()
	{
		// LoadModule has to be done on Main thread
		// can't be done on-demand in the Compress call
		
		// Make sure latest can be found up front.
		const FAstcEncThunk* DecodeThunk = LoadAstcVersion(SupportedAstcEncVersions[SupportedAstcEncVersionCount-1]);
		if (!DecodeThunk)
		{
			UE_LOGF(LogTextureFormatASTC, Fatal, "Couldn't find latest ASTC enc version lib: %ls", AstcEncVersionStrings[SupportedAstcEncVersionCount-1]);
		}
	}

	virtual ~FTextureFormatASTC()
	{
		for (size_t i = 0; i < SupportedAstcEncVersionCount; i++)
		{
			if (AstcVersions[i].LibHandle)
			{
				FPlatformProcess::FreeDllHandle(AstcVersions[i].LibHandle);
				AstcVersions[i].LibHandle = 0;
			}
		}

	}

	static FGuid GetDecodeBuildFunctionVersionGuid()
	{
		static FGuid Version(TEXT("0520C2CC-FD1D-48FE-BDCB-4E6E07E01E5B"));
		return Version;
	}
	static FUtf8StringView GetDecodeBuildFunctionNameStatic()
	{
		return UTF8TEXTVIEW("FDecodeTextureFormatASTC");
	}
	virtual const FUtf8StringView GetDecodeBuildFunctionName() const override final
	{
		return GetDecodeBuildFunctionNameStatic();
	}

	virtual bool SupportsEncodeSpeed(FName, const ITargetPlatformSettings* TargetPlatform) const override
	{
		// We can't do this on construct because the target platforms aren't set up yet, so we have to do
		// this once we need the info.
		static bool bIsInitialized = [this]() mutable
		{
			// save off which platforms we use RDO for.
			const TArray<ITargetPlatformSettings*>& TargetPlatforms = GetTargetPlatformManagerRef().GetTargetPlatformSettings();
			for (const ITargetPlatformSettings* TargetPlatform : TargetPlatforms)
			{
				// Platforms get added multiple times due to different shader possibilities or whatnot.
				if (!RDOEnabledByPlatform.Contains(TargetPlatform->IniPlatformName()))
				{
					bool bRDOEnabled = false;
					const FString& SectionName = FDataDrivenPlatformInfoRegistry::GetPlatformInfo(TargetPlatform->IniPlatformName()).TargetSettingsIniSectionName;

					TargetPlatform->GetConfigSystem()->GetBool(*SectionName, TEXT("bASTCUseRDO"), bRDOEnabled, GEngineIni);

					RDOEnabledByPlatform.Add(TargetPlatform->IniPlatformName(), bRDOEnabled);

					if (bRDOEnabled)
					{
						UE_LOGF(LogTextureFormatASTC, Display, "ArmASTC RDO: %ls from section %ls on platform %ls", 
							bRDOEnabled ? TEXT("enabled") : TEXT("disabled"), 
							*SectionName, *WriteToString<40>(TargetPlatform->IniPlatformName()));
					}
				}
			}

			return true;
		}();

		// Returning true causes UE to resolve the RDO settings that we want for our own RDO, because the RDO settings
		// happen to be stored in the encode speed block.
		const bool* RDOEnabledPtr = RDOEnabledByPlatform.Find(TargetPlatform->IniPlatformName());
		if (RDOEnabledPtr && *RDOEnabledPtr)
		{
			return true;
		}
		return false;
	}

	virtual bool AllowParallelBuild() const override
	{
		return true;
	}
	virtual FName GetEncoderName(FName Format) const override
	{
		static const FName ASTCName("ArmASTC");
		return ASTCName;
	}

	virtual FCbObject ExportGlobalFormatConfig(const FTextureBuildSettings& BuildSettings) const override
	{
		FCbWriter Writer;
		Writer.BeginObject("TextureFormatASTCSettings");
		Writer.AddInteger("DefaultASTCQualityBySize", GetDefaultCompressionBySizeValue(FCbObjectView()));
		Writer.AddInteger("DefaultASTCQualityBySizeHQ", GetDefaultCompressionBySizeValueHQ(FCbObjectView()));
		Writer.AddInteger("DefaultASTCQualityBySpeed", GetDefaultCompressionBySpeedValue(FCbObjectView()));
		Writer.EndObject();
		return Writer.Save().AsObject();
	}

	// Version for all ASTC textures
	virtual uint16 GetVersion(
		FName Format,
		const FTextureBuildSettings* BuildSettings = nullptr
	) const override
	{
		return BASE_ASTC_FORMAT_VERSION;
	}

	virtual FString GetDerivedDataKeyString(const FTextureBuildSettings& InBuildSettings, int32 InMipCount, const FIntVector3& InMip0Dimensions) const override
	{
		// ASTC block size chosen is in PixelFormat
		EPixelFormat PixelFormat = GetQualityFormat(InBuildSettings);
		int Speed = GetDefaultCompressionBySpeedValue(InBuildSettings.FormatConfigOverride);

		TStringBuilder<64> ASTCSuffix;
		ASTCSuffix << TEXT("ASTC_");
		ASTCSuffix << (int)PixelFormat;
		ASTCSuffix << TEXT("_");
		ASTCSuffix << Speed;

		// we don't support RDO for HDR
		if (IsRDOEncode(InBuildSettings))
		{
			ASTCSuffix << TEXT("RDO_") << InBuildSettings.OodleRDO;
		}

		// only add in the version if we aren't the first version (4.2.0). None defaults to first version.
		if (InBuildSettings.AstcEncVersion != NAME_None &&
			InBuildSettings.AstcEncVersion != SupportedAstcEncVersions[0])
		{
			ASTCSuffix << TEXT("V_") << InBuildSettings.AstcEncVersion;
		}

		return FString(ASTCSuffix);
	}

	virtual void GetSupportedFormats(TArray<FName>& OutFormats) const override
	{
		OutFormats.Append(GSupportedTextureFormatNames, sizeof(GSupportedTextureFormatNames)/sizeof(GSupportedTextureFormatNames[0]) ); 
	}

	virtual EPixelFormat GetEncodedPixelFormat(const FTextureBuildSettings& InBuildSettings, bool bInImageHasAlphaChannel) const override
	{
		return GetQualityFormat(InBuildSettings);
	}


	virtual bool CanDecodeFormat(EPixelFormat InPixelFormat) const
	{
		return IsASTCBlockCompressedTextureFormat(InPixelFormat);
	}

	virtual bool DecodeImage(int32 InSizeX, int32 InSizeY, int32 InNumSlices, EPixelFormat InPixelFormat, bool bInSRGB, const FName& InTextureFormatName, FSharedBuffer InEncodedData, FImage& OutImage, FStringView InTextureName) const
	{
		// We require the latest version to be available.
		const FAstcEncThunk* DecodeThunk = LoadAstcVersion(SupportedAstcEncVersions[SupportedAstcEncVersionCount-1]);

		FAstcEncThunk_CreateParams CreateParams;

		bool bHDRImage = IsASTCPixelFormatHDR(InPixelFormat);

		CreateParams.Profile = (bHDRImage ? EAstcEncThunk_Profile::HDR_RGB_LDR_A : (bInSRGB ? EAstcEncThunk_Profile::LDR_SRGB : EAstcEncThunk_Profile::LDR));
		CreateParams.BlockSize = GPixelFormats[InPixelFormat].BlockSizeX;
		CreateParams.Quality = EAstcEncThunk_Quality::THOROUGH;
		CreateParams.Flags = EAstcEncThunk_Flags::DECOMPRESS_ONLY;
		CreateParams.TaskCount = 1;

		{
			CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_R;
			CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
			CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_B;
			CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_A;

			if (IsASTCPixelFormatHDR(InPixelFormat))
			{
				// BC6H, our compressed HDR format on non-ASTC targets, does not support A
				CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
			}
			else
			{
				// Check for the other variants individually here
				// set everything up with normal (RGBA) swizzles
				if (InTextureFormatName == GTextureFormatNameASTC_NormalAG)
				{
					CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_A;
					CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
					CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_0;
					CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
				}
				else if (InTextureFormatName == GTextureFormatNameASTC_NormalRG)
				{
					CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_R;
					CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_G;
					CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_0;
					CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
				}
				else if (InTextureFormatName == GTextureFormatNameASTC_NormalLA || InTextureFormatName == GTextureFormatNameASTC_NormalRG_Precise)
				{
					CreateParams.SwizzleR = EAstcEncThunk_SwizzleComp::SELECT_R;
					CreateParams.SwizzleG = EAstcEncThunk_SwizzleComp::SELECT_A;
					CreateParams.SwizzleB = EAstcEncThunk_SwizzleComp::SELECT_0;
					CreateParams.SwizzleA = EAstcEncThunk_SwizzleComp::SELECT_1;
				}

				// Finally, last step, because ASTCEnc produces RGBA channel order and we want BGRA for 8-bit formats:
				Swap(CreateParams.SwizzleR, CreateParams.SwizzleB);
			}
		}

		// astc image basically wants views into the image but also wants them as an array of pointers
		// to each slice.
		TArray<uint8*, TInlineAllocator<6>> ImageSrcData;
		{
			OutImage.Format = bHDRImage ? ERawImageFormat::RGBA16F : ERawImageFormat::BGRA8;
			OutImage.GammaSpace = bInSRGB ? EGammaSpace::sRGB : EGammaSpace::Linear;
			OutImage.SizeX = InSizeX;
			OutImage.SizeY = InSizeY;
			OutImage.NumSlices = InNumSlices;

			const FPixelFormatInfo& OutputPF = GPixelFormats[bHDRImage ? PF_FloatRGBA : PF_B8G8R8A8];

			uint64 SliceSizeBytes = OutputPF.Get2DImageSizeInBytes(InSizeX, InSizeY);

			OutImage.RawData.AddUninitialized(SliceSizeBytes * InNumSlices);

			ImageSrcData.Reserve(OutImage.NumSlices);
			for (int32 SliceIdx = 0; SliceIdx < OutImage.NumSlices; SliceIdx++)
			{
				FImageView Slice = OutImage.GetSlice(SliceIdx);
				uint8* SliceData;
				if (bHDRImage)
				{
					SliceData = (uint8*)Slice.AsRGBA16F().GetData();
				}
				else
				{
					SliceData = (uint8*)Slice.AsBGRA8().GetData();
				}
				ImageSrcData.Add(SliceData);
			}
		}

		CreateParams.ImageSlices = (void**)ImageSrcData.GetData();
		CreateParams.SizeX = OutImage.SizeX;
		CreateParams.SizeY = OutImage.SizeY;
		CreateParams.NumSlices = OutImage.NumSlices;
		CreateParams.ImageDataType = (bHDRImage ? EAstcEncThunk_Type::F16 : EAstcEncThunk_Type::U8);

		CreateParams.OutputImageBuffer = (uint8_t*)InEncodedData.GetData();
		CreateParams.OutputImageBufferSize = InEncodedData.GetSize();


		AstcEncThunk_Context Context;
		const char* Error = DecodeThunk->Create(CreateParams, &Context);

		if (!Error)
		{
			Error = DecodeThunk->DoWork(Context, 0);
		}

		DecodeThunk->Destroy(Context);

		if (Error)
		{
			UE_LOGF(LogTextureFormatASTC, Error, "Failed to decode astc image: %ls - texture %.*ls", ANSI_TO_TCHAR(Error), InTextureName.Len(), InTextureName.GetData());
			return false;
		}

		return true;
	}


	virtual bool CompressImage(
			const FImage& InImage,
			const FTextureBuildSettings& BuildSettings,
			const FIntVector3& InMip0Dimensions,
			int32 InMip0NumSlicesNoDepth,
			int32 InMipIndex,
			int32 InMipCount,
			FStringView DebugTexturePathName,
			bool bImageHasAlphaChannel,
			FCompressedImage2D& OutCompressedImage
		) const override
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(ASTC.CompressImage);

		UE_CALL_ONCE( [&](){
			UE_LOGF(LogTextureFormatASTC, Display, "TextureFormatASTC using astcenc")
		} );

		const FAstcEncThunk* Thunk = LoadAstcVersion(BuildSettings.AstcEncVersion);
		// We can't fall back because we have the version in the DDC key.
		if (!Thunk)
		{
			return false;
		}

		return ASTCEnc_Compress(Thunk, InImage, BuildSettings, InMip0Dimensions, InMip0NumSlicesNoDepth, DebugTexturePathName, bImageHasAlphaChannel, OutCompressedImage);
	}

private:

	const FAstcEncThunk* LoadAstcVersion(FName Version) const
	{
		// None always maps to the first version we support with this.
		if (Version == NAME_None)
		{
			Version = SupportedAstcEncVersions[0];
		}

		for (size_t i = 0; i < SupportedAstcEncVersionCount; i++)
		{
			if (SupportedAstcEncVersions[i] == Version)
			{
				FAstcEncThunk* ThunkVersion = (FAstcEncThunk*)&AstcVersions[i];

				UE::TUniqueLock Lock(ThunkVersion->LoaderLock);
				if (ThunkVersion->bHasAttemptedLoad)
				{
					return ThunkVersion->Create ? ThunkVersion : nullptr;
				}
				
				ThunkVersion->bHasAttemptedLoad = true;

				// Try to load.
				TStringBuilder<128> DllName;
				DllName << ASTCENC_DLL_PREFIX << AstcEncVersionStrings[i] << ASTCENC_DLL_SUFFIX;

				ThunkVersion->LibHandle = FPlatformProcess::GetDllHandle(DllName.ToString());
				if (!ThunkVersion->LibHandle)
				{
					if (i != SupportedAstcEncVersionCount-1)
					{
						UE_LOGF(LogTextureFormatASTC, Warning, "ASTCEnc version %ls requested but not found.", AstcEncVersionStrings[i]);
					}
					else
					{
						UE_LOGF(LogTextureFormatASTC, Fatal, "Latest ASTCEnc version %ls required but not found.", AstcEncVersionStrings[i]);
					}
					return nullptr;
				}

				ThunkVersion->Create = (AstcThunk_CreateFnType*)FPlatformProcess::GetDllExport(ThunkVersion->LibHandle, TEXT("AstcEncThunk_Create"));
				ThunkVersion->Destroy = (AstcThunk_DestroyFnType*)FPlatformProcess::GetDllExport(ThunkVersion->LibHandle, TEXT("AstcEncThunk_Destroy"));
				ThunkVersion->DoWork = (AstcThunk_DoWorkFnType*)FPlatformProcess::GetDllExport(ThunkVersion->LibHandle, TEXT("AstcEncThunk_DoWork"));

				AstcThunk_SetAllocatorsFnType* SetAllocators = (AstcThunk_SetAllocatorsFnType*)FPlatformProcess::GetDllExport(ThunkVersion->LibHandle, TEXT("AstcEncThunk_SetAllocators"));

				// we require all function pointers - if we didn't get them all, it's a corrupted dll and
				// we are bound to crash later.
				if (!ThunkVersion->Create ||
					!ThunkVersion->Destroy || 
					!ThunkVersion->DoWork ||
					!SetAllocators)
				{
					UE_LOGF(LogTextureFormatASTC, Fatal, "ASTCEnc version %ls library loaded but has missing exports", AstcEncVersionStrings[i]);
					return nullptr;
				}

				UE_LOGF(LogTextureFormatASTC, Display, "ASTCEnc version %ls library loaded", AstcEncVersionStrings[i]);

				SetAllocators(FMemory_AstcThunk_Malloc, FMemory_AstcThunk_Free);
				return ThunkVersion;
			}
		}

		return nullptr;
	}

	static constexpr size_t SupportedAstcEncVersionCount = 2;
	FName SupportedAstcEncVersions[SupportedAstcEncVersionCount] = 
	{
		FName(TEXT("420")),
		FName(TEXT("501"))
	};
	const TCHAR* AstcEncVersionStrings[SupportedAstcEncVersionCount] = 
	{
		TEXT("4.2.0"),
		TEXT("5.0.1")
	};

	FAstcEncThunk AstcVersions[SupportedAstcEncVersionCount];

	mutable TMap<FString, bool> RDOEnabledByPlatform;
};

/**
 * Module for ASTC texture compression.
 */
static ITextureFormat* Singleton = NULL;

class FTextureFormatASTCModule : public ITextureFormatModule
{
public:
	FTextureFormatASTCModule()
	{
	}
	virtual ~FTextureFormatASTCModule()
	{
		delete Singleton;
		Singleton = NULL;
	}
	
	virtual void StartupModule() override
	{
	}

	virtual bool CanCallGetTextureFormats() override { return false; }

	virtual ITextureFormat* GetTextureFormat()
	{
		if (!Singleton)
		{
			Singleton = new FTextureFormatASTC();
		}
		return Singleton;
	}

	static inline UE::DerivedData::TBuildFunctionFactory<FASTCTextureBuildFunction> BuildFunctionFactory;
	static inline UE::DerivedData::TBuildFunctionFactory<FGenericTextureDecodeBuildFunction<FTextureFormatASTC>> DecodeBuildFunctionFactory;
};

IMPLEMENT_MODULE(FTextureFormatASTCModule, TextureFormatASTC);
