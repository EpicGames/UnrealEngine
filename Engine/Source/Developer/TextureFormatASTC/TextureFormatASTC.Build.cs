// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class TextureFormatASTC : ModuleRules
{
	public TextureFormatASTC(ReadOnlyTargetRules Target) : base(Target)
	{
		PrivateIncludePathModuleNames.AddRange(new string[]
		{
			"TargetPlatform",
			"TextureCompressor",
			"TextureFormat",
		});
		
		PrivateDependencyModuleNames.AddRange(new string[]
		{
			"Core",
			"DerivedDataCache", // for FCacheBucket
			"ImageCore",
			"TextureBuild",
			"astcenc",
		});
	}
}
